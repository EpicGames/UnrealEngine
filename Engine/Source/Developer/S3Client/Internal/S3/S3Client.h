// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (IS_PROGRAM || WITH_EDITOR)

#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Memory/MemoryView.h"
#include "Memory/SharedBuffer.h"
#include "Misc/DateTime.h"
#include "Misc/StringBuilder.h"

#define UE_API S3CLIENT_API

namespace UE
{

/** Credentials for creating signed/authenticated requests to S3. */
class FS3ClientCredentials
{
public:
	/** Default constructor creating invalid credentails. */
	FS3ClientCredentials() = default;

	/**
	 * Constructor for basic credentials.
	 *
	 * @param AccessKey			The access key.
	 * @param SecretKey			The secret key.
	 */
	FS3ClientCredentials(const FString& InAccessKey, const FString& InSecretKey)
		: AccessKey(InAccessKey)
		, SecretKey(InSecretKey)
	{
	}

	/**
	 * Constructor for short termed credentials.
	 *
	 * @param AccessKey			The access key.
	 * @param SecretKey			The secret key.
	 * @param SessionToken		Token defining the lifetime of the credentials.
	 */
	FS3ClientCredentials(const FString& InAccessKey, const FString& InSecretKey, const FString& InSessionToken)
		: AccessKey(InAccessKey)
		, SecretKey(InSecretKey)
		, SessionToken(InSessionToken)
	{
	}

	FS3ClientCredentials(const FS3ClientCredentials& Other) = default;
	FS3ClientCredentials(FS3ClientCredentials&&) = default;
	FS3ClientCredentials& operator=(const FS3ClientCredentials&) = default;
	FS3ClientCredentials& operator=(FS3ClientCredentials&&) = default;

	/** Returns whether the access and secret is is valid. */
	bool IsValid() const;
	/** Returns the access key. */
	const FString& GetAccessKey() const { return AccessKey; }
	/** Returns the secret key. */
	const FString& GetSecretKey() const { return SecretKey; }
	/** Returns the session token. */
	const FString& GetSessionToken() const { return SessionToken; }

private:
	FString AccessKey;
	FString SecretKey;
	FString SessionToken;
};

/**
 * Container for named credentials.
 */
class FS3CredentialsProfileStore
{
public:
	/** Returns the first credentials loaded into the profile store. */
	UE_API FS3ClientCredentials GetDefault() const;
	/** Get credentials for the specified profile name. */
	UE_API bool TryGetCredentials(const FString& ProfileName, FS3ClientCredentials& OutCredentials) const;
	/** Reads named credentials from an .ini file. */
	static UE_API FS3CredentialsProfileStore FromFile(const FString& FileName, FString* OutError = nullptr);

private:
	TMap<FString, FS3ClientCredentials> Credentials;
};

/** Describes an object stored in S3. */
struct FS3Object
{
	/** The object identifider. */
	FString Key;
	/** Date and time when this object was last modified in text format. */
	FString LastModifiedText;
	/** Date and time when this object was last modified. */
	FDateTime LastModified;
	/** The size of the object. */
	uint64 Size = 0;

	bool operator==(const FS3Object& RHS) const { return Key == RHS.Key; }
	bool operator!=(const FS3Object& RHS) const { return !operator==(RHS); }

	friend inline uint32 GetTypeHash(const FS3Object& Object)
	{
		return GetTypeHash(Object.Key);
	}
};

/** Basic response parameters. */
struct FS3Response
{
public:

	FS3Response() = default;
	
	FS3Response(const FS3Response& Other);
	FS3Response(FS3Response&& Other);

	FS3Response(uint32 InHttpStatusCode, uint32 InApiStatusCode);
	FS3Response(uint32 InHttpStatusCode, uint32 InApiStatusCode, FSharedBuffer&& InBody);
	FS3Response(uint32 InHttpStatusCode, FS3Response&& Other);

public:

	~FS3Response() = default;

	UE_API FS3Response& operator=(const FS3Response& Other);
	UE_API FS3Response& operator=(FS3Response&& Other);

	/** Returns whether the request is considered successful or not. */
	UE_API bool IsOk() const;

	/** Returns the body as text. */
	UE_API FString ToString() const;

	/** Returns the body as a raw buffer */
	UE_API FSharedBuffer GetBody() const;

	/** 
	 * Returns an error message comprised of the api/http status code and any error
	 * that might have been returned as part of the response body.
	 * If there response had no errors it the message will be "Success"
	 */
	UE_API void GetErrorResponse(FStringBuilderBase& OutErrorMsg) const;
	
	/** 
	 * Returns a short error message comprised of the api/http status code
	 * If there response had no errors it the message will be "Success"
	 */
	UE_API FString GetErrorStatus() const;

private:

	/** Status code returned by the HTTP request. */
	uint32 HttpStatusCode = 0;

	/** Status code returned by the API */
	uint32 ApiStatusCode = 0;

	/** HTTP response body. */
	FSharedBuffer Body;
};

/** Request parameters for retrieving objects. */
struct FS3GetObjectRequest
{
	/** The bucket name. */
	FString BucketName;
	/** The object key. */
	FString Key;
};

/** Response parameters when retrieving objects. */
using FS3GetObjectResponse = FS3Response;

/** Request parameters for retrieving object meta data. */
struct FS3HeadObjectRequest
{
	/** The bucket name. */
	FString BucketName;
	/** The object key. */
	FString Key;
};

/** Response parameters when retrieving object meta data. */
using FS3HeadObjectResponse = FS3Response;

/** Request parameters for uploading objects. */
struct FS3PutObjectRequest
{
	/** The bucket name. */
	FString BucketName;
	/** The object key. */
	FString Key;
	/** The object data. */
	FMemoryView ObjectData;
};

/** Response parameters when uploading objects. */
using FS3PutObjectResponse = FS3Response;

/** Request parameters for listing objects. */
struct FS3ListObjectsRequest
{
	/** The bucket name. */
	FString BucketName;
	/** The object prefix, i.e. the path. */
	FString Prefix;
	/** The path delimiter, i.e. '/'. */
	TCHAR Delimiter;
	/* Max key(s). */
	int32 MaxKeys = INDEX_NONE;
	/** Marker from where to list objects. */
	FString Marker;
};

/** Response parameters when listing objects. */
struct FS3ListObjectResponse
	: public FS3Response
{
	FS3ListObjectResponse() = default;

	FS3ListObjectResponse(FS3Response&& Other)
		: FS3Response(MoveTemp(Other))
	{}

	FS3ListObjectResponse(FS3Response&& Other, FString&& InBucketName, TArray<FS3Object>&& InObjects, FString InNextMarker = FString(), bool bInIsTruncated = false)
		: FS3Response(MoveTemp(Other))
		, BucketName(MoveTemp(InBucketName))
		, Objects(MoveTemp(InObjects))
		, NextMarker(MoveTemp(InNextMarker))
		, bIsTruncated(bInIsTruncated)
	{}

	/** The bucket name. */
	FString BucketName;
	/** The list of object(s). */
	TArray<FS3Object> Objects;
	/** Marker to use for paginated requests. */
	FString NextMarker;
	/** Whether the response is truncated. */
	bool bIsTruncated = false;
};

/** Request parameters for deleting objects. */
using FS3DeleteObjectRequest = FS3GetObjectRequest;

/** Response parameters when deleting objects. */
using FS3DeleteObjectResponse = FS3Response;

/** S3 client configuration with region and service URL. */
struct FS3ClientConfig
{
	FString Region; 
	FString ServiceUrl;
};

/**
 * A simple HTTP(s) client for down, uploading and listing data objects from Amazon S3.
 */
class FS3Client
{
	class FConnectionPool;
	class FS3Request;

public:
	/** Creates a new instance from the specified configuration and credentials. */
	UE_API explicit FS3Client(const FS3ClientConfig& ClientConfig, const FS3ClientCredentials& BasicCredentials);
	FS3Client(const FS3Client&) = delete;
	FS3Client(FS3Client&&) = delete;
	UE_API ~FS3Client();

	FS3Client& operator=(const FS3Client&) = delete;
	FS3Client& operator=(FS3Client&&) = delete;

	/** Returns the credentials. */
	const FS3ClientCredentials& GetCredentials() const { return Credentials; }
	/** Returns the client configuration. */
	const FS3ClientConfig& GetConfig() const { return Config; }

	/** Download an object associated with the specified request parameters. */
	UE_API FS3GetObjectResponse GetObject(const FS3GetObjectRequest& Request);
	/** Download object meta data. */
	UE_API FS3HeadObjectResponse HeadObject(const FS3HeadObjectRequest& Request);
	/** List all objects described by the specified request parameters. */
	UE_API FS3ListObjectResponse ListObjects(const FS3ListObjectsRequest& Request);
	/** Upload an object described by the specified request parameters. */
	UE_API FS3PutObjectResponse PutObject(const FS3PutObjectRequest& Request);
	/** Retries uploading an object until succeeded or max attempts has been reached. */
	UE_API FS3PutObjectResponse TryPutObject(const FS3PutObjectRequest& Request, int32 MaxAttempts = 3, float Delay = 0.5f);
	/** Delete an object associated with the specified request parameters. */
	UE_API FS3DeleteObjectResponse DeleteObject(const FS3DeleteObjectRequest& Request);

private:
	UE_API void Setup(FS3Request& Request);
	UE_API void Teardown(FS3Request& Request);

	FS3ClientConfig Config;
	FS3ClientCredentials Credentials;
	TUniquePtr<FConnectionPool> ConnectionPool;
};

} // namesapce UE

#endif // (IS_PROGRAM || WITH_EDITOR)

#undef UE_API
