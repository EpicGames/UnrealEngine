// Copyright Epic Games, Inc. All Rights Reserved.

#include "S3/S3Client.h"
#include "Containers/ChunkedArray.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"

#if (IS_PROGRAM || WITH_EDITOR)

#include "Containers/StringConv.h"
#include "Misc/CString.h"
#include "Misc/DateTime.h"
#include "Serialization/LargeMemoryWriter.h"
#include "String/BytesToHex.h"
#include "XmlParser.h"

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif
#ifdef PLATFORM_CURL_INCLUDE
#include PLATFORM_CURL_INCLUDE
#else
#include "curl/curl.h"
#endif
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

#include "Ssl.h"
#include <openssl/hmac.h>
#include <openssl/sha.h>
#include <openssl/ssl.h>

IMPLEMENT_MODULE(FDefaultModuleImpl, S3Client);

DEFINE_LOG_CATEGORY_STATIC(LogS3Client, Log, All);

// Enable to start logging debug output from curl
#define UE_ENABLE_S3_DEBUGGING 0

namespace UE
{

namespace S3Client
{

#if UE_ENABLE_S3_DEBUGGING

int DebugCallback(CURL* Handle, curl_infotype Type, char* Data, size_t Size, void* UserData)
{
	FString Message(Size, Data);
	Message.TrimEndInline();

	switch (Type)
	{
		case CURLINFO_TEXT:
			UE_LOGF(LogS3Client, Display, "Info: %ls", *Message);
			break;
		case CURLINFO_HEADER_OUT:
			UE_LOGF(LogS3Client, Display, "Send Header: %ls", *Message);
			break;
		case CURLINFO_HEADER_IN:
			UE_LOGF(LogS3Client, Display, "Recv Header: %ls", *Message);
			break;
		default:
			break;
	}

	return 0;
}

#endif // UE_ENABLE_S3_DEBUGGING

////////////////////////////////////////////////////////////////////////////////
static FAnsiStringBuilderBase& AppendUrl(
	FAnsiStringBuilderBase& Url,
	const FString& Host,
	const FString& BucketName,
	const FString& Key)
{
	check(!Host.IsEmpty() && !Key.IsEmpty());

	Url << StringCast<ANSICHAR>(*Host);
	if (!BucketName.IsEmpty())
	{
		Url << "/" << StringCast<ANSICHAR>(*BucketName);
	}
	Url << "/" << StringCast<ANSICHAR>(*Key);

	return Url;
}

////////////////////////////////////////////////////////////////////////////////
struct FSHA256
{
	using ByteArray = uint8[32];
	
	FSHA256() = default;
	
	void ToString(FAnsiStringBuilderBase& Sb)
	{
		UE::String::BytesToHexLower(Hash, Sb);
	}

	alignas(uint32) ByteArray Hash{};
};

FSHA256 Sha256(const uint8* Input, size_t InputLen)
{
	FSHA256 Output;
	SHA256(Input, InputLen, Output.Hash);
	return Output;
}

FSHA256 HmacSha256(const uint8* Input, size_t InputLen, const uint8* Key, size_t KeyLen)
{
	FSHA256 Output;
	unsigned int OutputLen = 0;
	HMAC(EVP_sha256(), Key, KeyLen, (const unsigned char*)Input, InputLen, Output.Hash, &OutputLen);
	return Output;
}

FSHA256 HmacSha256(const char* Input, const uint8* Key, size_t KeyLen)
{
	return HmacSha256((const uint8*)Input, (size_t)FCStringAnsi::Strlen(Input), Key, KeyLen);
}

} // namespace UE::S3

////////////////////////////////////////////////////////////////////////////////
struct FCurlHandle
{
	CURL* operator*() { return Handle; }
	operator CURL*() { return Handle; }

	CURL* Handle = nullptr;
	int32 PoolIndex = INDEX_NONE;
};

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* UrlEncode(FCurlHandle& Curl, FAnsiStringView String, FAnsiStringBuilderBase& Out)
{
	check(!String.IsEmpty());

	ANSICHAR* Encoded = curl_easy_escape(Curl, *WriteToAnsiString<64>(String), String.Len());
	Out.Reset();	
	Out.Append(Encoded, FCStringAnsi::Strlen(Encoded));
	curl_free(Encoded);

	return Out.ToString();
}

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* GetAuthorizationHeader(
	FCurlHandle& Curl,
	const FS3Client& Client,
	const ANSICHAR* Verb,
	const ANSICHAR* RelativeUrl,
	const ANSICHAR* QueryString,
	const curl_slist* Headers,
	const ANSICHAR* Timestamp,
	const ANSICHAR* Digest,
	FAnsiStringBuilderBase& Out)
{
	using namespace UE::S3Client;

	//https://docs.aws.amazon.com/AmazonS3/latest/API/sig-v4-header-based-auth.html
	
	//TODO: Remove temporary strings

	const FS3ClientCredentials& Credentials = Client.GetCredentials();
	const FS3ClientConfig& Config = Client.GetConfig();

	// Create the canonical URL w/o the query parameter(s)
	TAnsiStringBuilder<128> CanonicalUrl;
	CanonicalUrl << RelativeUrl;
	if (const ANSICHAR* QuestionMark = FCStringAnsi::Strchr(RelativeUrl, '?'); QuestionMark != nullptr)
	{
		CanonicalUrl.Reset();
		CanonicalUrl.Append(RelativeUrl, UE_PTRDIFF_TO_INT32(QuestionMark - RelativeUrl));
	}

	// Create the canonical list of headers
	TAnsiStringBuilder<256> CanonicalHeaders;
	for (const curl_slist* Header = Headers; Header != nullptr; Header = Header->next)
	{
		const ANSICHAR* Colon = FCStringAnsi::Strchr(Header->data, ':');
		if (Colon != nullptr)
		{
			for (const ANSICHAR* Char = Header->data; Char != Colon; Char++)
			{
				CanonicalHeaders.AppendChar(FCharAnsi::ToLower(*Char));
			}
			CanonicalHeaders.AppendChar(':');

			const ANSICHAR* Value = Colon + 1;
			while (*Value == ' ')
			{
				Value++;
			}
			for (; *Value != 0; Value++)
			{
				CanonicalHeaders.AppendChar(*Value);
			}
			CanonicalHeaders.AppendChar('\n');
		}
	}

	// Create the list of signed headers
	TAnsiStringBuilder<256> SignedHeaders;
	for (const curl_slist* Header = Headers; Header != nullptr; Header = Header->next)
	{
		const ANSICHAR* Colon = FCStringAnsi::Strchr(Header->data, ':');
		if (Colon != nullptr)
		{
			if (SignedHeaders.Len() > 0)
			{
				SignedHeaders.AppendChar(';');
			}
			for (const ANSICHAR* Char = Header->data; Char != Colon; Char++)
			{
				SignedHeaders.AppendChar(FCharAnsi::ToLower(*Char));
			}
		}
	}

	//TODO: Sort the parameters
	// Build the canonical query string
	TAnsiStringBuilder<128> CanonicalQueryString;
	{
		FAnsiStringView Query(QueryString);
		bool bFirst = true;
		while (!Query.IsEmpty())
		{
			FAnsiStringView Token = Query;

			int32 Idx = MAX_int32;
			if (Query.FindChar(ANSICHAR('&'), Idx))
			{
				Token.LeftInline(Idx);
				Query.RightChopInline(Idx + 1);
			}
			else
			{
				Query = FAnsiStringView();
			}

			Idx = INDEX_NONE;
			ensure(Token.FindChar(ANSICHAR('='), Idx));

			FAnsiStringView Param = Token.Left(Idx);
			FAnsiStringView Value = Token.RightChop(Idx + 1);

			if (!bFirst)
			{
				CanonicalQueryString.AppendChar('&');
			}

			TAnsiStringBuilder<64> Tmp;
			CanonicalQueryString.Append(UrlEncode(Curl, Param, Tmp));
			CanonicalQueryString.AppendChar('=');
			CanonicalQueryString.Append(UrlEncode(Curl, Value, Tmp));

			bFirst = false;
		}
	}

	// Build the canonical request string
	TAnsiStringBuilder<1024> CanonicalRequest;
	CanonicalRequest.Append(Verb);
	CanonicalRequest.AppendChar('\n');
	CanonicalRequest.Append(CanonicalUrl);
	CanonicalRequest.AppendChar('\n');
	CanonicalRequest.Append(CanonicalQueryString);
	CanonicalRequest.AppendChar('\n');
	CanonicalRequest.Append(CanonicalHeaders);
	CanonicalRequest.AppendChar('\n');
	CanonicalRequest.Append(SignedHeaders);
	CanonicalRequest.AppendChar('\n');
	CanonicalRequest.Append(Digest);

	// Get the date
	TAnsiStringBuilder<32> DateString;
	for (int32 Idx = 0; Timestamp[Idx] != 0 && Timestamp[Idx] != 'T'; Idx++)
	{
		DateString.AppendChar(Timestamp[Idx]);
	}

	// Generate the signature key
	TAnsiStringBuilder<64> Key;
	Key.Appendf("AWS4%s", StringCast<ANSICHAR>(*Credentials.GetSecretKey()).Get());

	FSHA256 DateHash = HmacSha256(*DateString, (const uint8*)*Key, Key.Len());
	FSHA256 RegionHash = HmacSha256(StringCast<ANSICHAR>(*Config.Region).Get(), DateHash.Hash, sizeof(DateHash.Hash));
	FSHA256 ServiceHash = HmacSha256("s3", RegionHash.Hash, sizeof(RegionHash.Hash));
	FSHA256 SigningKeyHash = HmacSha256("aws4_request", ServiceHash.Hash, sizeof(ServiceHash.Hash));

	// Calculate the signature
	TAnsiStringBuilder<64> DateRequest;
	DateRequest.Appendf("%s/%s/s3/aws4_request", *DateString, StringCast<ANSICHAR>(*Config.Region).Get());

	TAnsiStringBuilder<32> CanonicalRequestSha256; 
	Sha256((const uint8*)*CanonicalRequest, CanonicalRequest.Len()).ToString(CanonicalRequestSha256);

	TAnsiStringBuilder<256> StringToSign; 
	StringToSign.Appendf("AWS4-HMAC-SHA256\n%s\n%s\n%s", Timestamp, *DateRequest, *CanonicalRequestSha256);
	
	TAnsiStringBuilder<32> Signature; 
	HmacSha256(*StringToSign, SigningKeyHash.Hash, sizeof(SigningKeyHash.Hash)).ToString(Signature);

	Out.Appendf("Authorization: AWS4-HMAC-SHA256 Credential=%s/%s, SignedHeaders=%s, Signature=%s",
		StringCast<ANSICHAR>(*Credentials.GetAccessKey()).Get(), *DateRequest, *SignedHeaders, *Signature);

	return Out.ToString();
}

////////////////////////////////////////////////////////////////////////////////
bool FS3ClientCredentials::IsValid() const
{
	return AccessKey.Len() > 0 && SecretKey.Len() > 0;
}

////////////////////////////////////////////////////////////////////////////////
FS3ClientCredentials FS3CredentialsProfileStore::GetDefault() const
{
	if (auto It = Credentials.CreateConstIterator(); It)
	{
		return It.Value();
	}

	return FS3ClientCredentials();
}

bool FS3CredentialsProfileStore::TryGetCredentials(const FString& ProfileName, FS3ClientCredentials& OutCredentials) const
{
	if (const FS3ClientCredentials* Entry = Credentials.Find(ProfileName))
	{
		OutCredentials = *Entry;
		return true;
	}

	return false;
}

FS3CredentialsProfileStore FS3CredentialsProfileStore::FromFile(const FString& FileName, FString* OutError)
{
	FS3CredentialsProfileStore ProfileStore;

	FConfigFile Config;
	Config.Read(FileName);
	for (auto KV : AsConst(Config))
	{
		const FString& ProfileName = KV.Key;
		const FConfigSection& Section = KV.Value;
		const FConfigValue* AccessKey = Section.Find(TEXT("aws_access_key_id"));
		const FConfigValue* SecretKey = Section.Find(TEXT("aws_secret_access_key"));

		if (AccessKey && SecretKey)
		{
			FS3ClientCredentials Credentials;
			if (const FConfigValue* SessionToken = Section.Find(TEXT("aws_session_token")))
			{
				Credentials = FS3ClientCredentials(AccessKey->GetValue(), SecretKey->GetValue(), SessionToken->GetValue());
			}
			else
			{
				Credentials = FS3ClientCredentials(AccessKey->GetValue(), SecretKey->GetValue());
			}

			ProfileStore.Credentials.Add(ProfileName, MoveTemp(Credentials));
		}
	}

	return ProfileStore;
}

////////////////////////////////////////////////////////////////////////////////

FS3Response::FS3Response(const FS3Response& Other) = default;

FS3Response::FS3Response(FS3Response&& Other)
	: HttpStatusCode(Other.HttpStatusCode)
	, ApiStatusCode(Other.ApiStatusCode)
	, Body(MoveTemp(Other.Body))
{

}

FS3Response::FS3Response(uint32 InHttpStatusCode, uint32 InApiStatusCode)
	: HttpStatusCode(InHttpStatusCode)
	, ApiStatusCode(InApiStatusCode)
{

}

FS3Response::FS3Response(uint32 InHttpStatusCode, uint32 InApiStatusCode, FSharedBuffer&& InBody)
	: HttpStatusCode(InHttpStatusCode)
	, ApiStatusCode(InApiStatusCode)
	, Body(MoveTemp(InBody))
{

}

FS3Response::FS3Response(uint32 InHttpStatusCode, FS3Response&& Other)
	: HttpStatusCode(InHttpStatusCode)
	, ApiStatusCode(Other.ApiStatusCode)
	, Body(MoveTemp(Other.Body))
{

}

FS3Response& FS3Response::operator=(const FS3Response& Other) = default;

FS3Response& FS3Response::operator=(FS3Response&& Other)
{
	HttpStatusCode = Other.HttpStatusCode;
	ApiStatusCode = Other.ApiStatusCode;
	Body = MoveTemp(Other.Body);

	return *this;
}

bool FS3Response::IsOk() const
{
	return ApiStatusCode == CURLE_OK && HttpStatusCode > 199 && HttpStatusCode < 299;
}

FSharedBuffer FS3Response::GetBody() const
{
	return Body;
}

FString FS3Response::ToString() const
{
	return Body.GetSize() > 0
		? FString::ConstructFromPtrSize(reinterpret_cast<const ANSICHAR*>(Body.GetData()), static_cast<int32>(Body.GetSize()))
		: FString();
}

void FS3Response::GetErrorResponse(FStringBuilderBase& OutErrorMsg) const
{
	OutErrorMsg.Reset();

	if (IsOk())
	{
		OutErrorMsg << TEXT("Successs");
		return;
	}

	if(ApiStatusCode != CURLE_OK)
	{
		OutErrorMsg << curl_easy_strerror(static_cast<CURLcode>(ApiStatusCode));
	}
	else
	{
		OutErrorMsg << TEXT("StatusCode: ") << HttpStatusCode << TEXT("Error: ");

		const FString BodyResponseString = ToString();

		FXmlFile XmlFile;
		if (!XmlFile.LoadFile(BodyResponseString, EConstructMethod::ConstructFromBuffer))
		{
			OutErrorMsg << TEXT("Unknown");
			return;
		}

		const FXmlNode* RootNode = XmlFile.GetRootNode();
		if (!RootNode)
		{
			OutErrorMsg << TEXT("Unknown");
			return;
		}

		const FXmlNode* CodeNode = RootNode->FindChildNode(TEXT("Code"));
		if (!CodeNode)
		{
			OutErrorMsg << TEXT("Unknown");
			return;
		}

		const FXmlNode* MessageNode = RootNode->FindChildNode(TEXT("Message"));
		if (!MessageNode)
		{
			OutErrorMsg << TEXT("Unknown");
			return;
		}

		OutErrorMsg << CodeNode->GetContent() << TEXT(": ") << MessageNode->GetContent();
	}
}

FString FS3Response:: GetErrorStatus() const
{
	TStringBuilder<256> ErrorStatus;

	if (IsOk())
	{
		ErrorStatus << TEXT("Successs");
	}
	else if (ApiStatusCode != CURLE_OK)
	{
		ErrorStatus << curl_easy_strerror(static_cast<CURLcode>(ApiStatusCode));
	}
	else
	{
		ErrorStatus << TEXT("StatusCode: ") << HttpStatusCode << TEXT("Error: ");
	}

	return FString(ErrorStatus);
}

////////////////////////////////////////////////////////////////////////////////
class FS3Client::FConnectionPool
{
public:
	FConnectionPool();
	~FConnectionPool();

	FCurlHandle Alloc();
	void Free(FCurlHandle Handle);
	void Empty();

private:
	TChunkedArray<FCurlHandle> Pool;
	TArray<int32> FreeList;
	FCriticalSection PoolCS;
};

FS3Client::FConnectionPool::FConnectionPool()
{
}

FS3Client::FConnectionPool::~FConnectionPool()
{
	Empty();
}

FCurlHandle FS3Client::FConnectionPool::Alloc()
{
	FScopeLock _(&PoolCS);

	if (FreeList.Num() > 0)
	{
		const int32 PoolIndex = FreeList.Pop();
		return Pool[PoolIndex];
	}
	else
	{
		const int32 PoolIndex = Pool.Add();
		FCurlHandle& CurlHandle = Pool[PoolIndex];
		CurlHandle.Handle = curl_easy_init();
		CurlHandle.PoolIndex = PoolIndex;

		return CurlHandle;
	}
}

void FS3Client::FConnectionPool::Free(FCurlHandle CurlHandle)
{
	check(CurlHandle.Handle != nullptr);
	curl_easy_reset(CurlHandle.Handle);

	FScopeLock _(&PoolCS);
	FreeList.Add(CurlHandle.PoolIndex);
}

void FS3Client::FConnectionPool::Empty()
{
	FScopeLock _(&PoolCS);

	for (int32 PoolIndex = 0; PoolIndex < Pool.Num(); ++PoolIndex)
	{
		FCurlHandle& CurlHandle = Pool[PoolIndex];
		check(CurlHandle.Handle != nullptr);
		curl_easy_cleanup(CurlHandle.Handle);
	}

	Pool.Empty();
	FreeList.Empty();
}

////////////////////////////////////////////////////////////////////////////////
class FS3Client::FS3Request
{
	friend class FS3Client;
public:
	enum class EMethod
	{
		Get,
		Put,
		Delete,
		Head
	};

	FS3Request(FS3Client& S3Client);
	~FS3Request();
	
	FS3Response Perform(EMethod Method, const ANSICHAR* Url, FSharedBuffer Body);

private:
	static int StatusCallback(void* Ptr, curl_off_t TotalDownloadSize, curl_off_t CurrentDownloadSize, curl_off_t TotalUploadSize, curl_off_t CurrentUploadSize);
	static size_t WriteHeadersCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);
	static size_t ReadBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);
	static size_t WriteBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData);
	static size_t SeekBodyCallback(void* UserData, curl_off_t Offset, int Origin);
	static int SslCertVerify(int PreverifyOk, X509_STORE_CTX* Context);
	static CURLcode SslContextCallback(CURL* curl, void* sslctx, void* parm);
	const ANSICHAR* LexToString(EMethod Method);

	FS3Client& Client;
	FCurlHandle Curl;
	TAnsiStringBuilder<64> Host;
	TAnsiStringBuilder<64> Domain;
	TAnsiStringBuilder<1024> ResponseHeader;
	FLargeMemoryWriter ResponseBody;
	FSharedBuffer RequestBody;
	size_t BytesSent = 0;
};

FS3Client::FS3Request::FS3Request(FS3Client& S3Client)
	: Client(S3Client)
{
	Client.Setup(*this);
}

FS3Client::FS3Request::~FS3Request()
{
	Client.Teardown(*this);
}

FS3Response FS3Client::FS3Request::Perform(EMethod Method, const ANSICHAR* Url, FSharedBuffer Body)
{
	using namespace UE::S3Client;

	RequestBody = Body;
	const uint64 ContentLength = Body.GetSize();
	check((Method == EMethod::Get || Method == EMethod::Head || Method == EMethod::Delete) || ContentLength > 0);

	// Find the host from the URL
	const ANSICHAR* ProtocolEnd = FCStringAnsi::Strchr(Url, ':');
	check(ProtocolEnd != nullptr && *(ProtocolEnd + 1) == '/' && *(ProtocolEnd + 2) == '/');

	const ANSICHAR* UrlHost = ProtocolEnd + 3;
	const ANSICHAR* UrlHostEnd = FCStringAnsi::Strchr(UrlHost, '/');
	check(UrlHostEnd != nullptr);

	Host.Append(UrlHost, UE_PTRDIFF_TO_INT32(UrlHostEnd - UrlHost));
	Domain.Append(Url, UE_PTRDIFF_TO_INT32(UrlHostEnd - Url));

	const ANSICHAR* QueryString = "";
	if (const ANSICHAR* QueryStringStart = FCStringAnsi::Strchr(Url, '?'))
	{
		QueryString = QueryStringStart + 1;
	}

	// Get the header strings
	FDateTime Timestamp = FDateTime::UtcNow();
	TAnsiStringBuilder<64> TimeString;
	TimeString.Appendf("%04d%02d%02dT%02d%02d%02dZ", Timestamp.GetYear(), Timestamp.GetMonth(), Timestamp.GetDay(), Timestamp.GetHour(), Timestamp.GetMinute(), Timestamp.GetSecond());

	// Payload string
	TAnsiStringBuilder<64> PayloadSha256;
	if (Body.GetSize() > 0)
	{
		Sha256(reinterpret_cast<const uint8*>(Body.GetData()), Body.GetSize()).ToString(PayloadSha256);
	}
	else
	{
		Sha256(nullptr, 0).ToString(PayloadSha256);
	}

	// Create the headers
	TAnsiStringBuilder<1024> AuthHeader;
	curl_slist* CurlHeaders = nullptr;
	CurlHeaders = curl_slist_append(CurlHeaders, *WriteToAnsiString<128>("Host: ", *Host));
	CurlHeaders = curl_slist_append(CurlHeaders, *WriteToAnsiString<128>("x-amz-content-sha256: ", *PayloadSha256));
	CurlHeaders = curl_slist_append(CurlHeaders, *WriteToAnsiString<128>("x-amz-date: ", *TimeString));

	if (!Client.Credentials.GetSessionToken().IsEmpty())
	{
		CurlHeaders = curl_slist_append(CurlHeaders, *WriteToAnsiString<512>("x-amz-security-token: ", *Client.Credentials.GetSessionToken()));
	}

	const ANSICHAR* MethodString = LexToString(Method);
	CurlHeaders = curl_slist_append(CurlHeaders, GetAuthorizationHeader(Curl, Client, MethodString, UrlHostEnd, QueryString, CurlHeaders, *TimeString, *PayloadSha256, AuthHeader));

	// Append the unsigned headers
	if (Method == EMethod::Put)
	{
		CurlHeaders = curl_slist_append(CurlHeaders, *WriteToAnsiString<32>("Content-Length: ", ContentLength));
		CurlHeaders = curl_slist_append(CurlHeaders, *WriteToAnsiString<64>("Content-Type: ", "application/octet-stream"));
	}

	// Setup the request
	curl_easy_reset(Curl);

#if UE_ENABLE_S3_DEBUGGING
	curl_easy_setopt(Curl, CURLOPT_DEBUGFUNCTION, S3Client::DebugCallback);
	curl_easy_setopt(Curl, CURLOPT_DEBUGDATA, nullptr);
	curl_easy_setopt(Curl, CURLOPT_VERBOSE, 1L);
#endif //UE_ENABLE_S3_DEBUGGING
	
	if (Method == EMethod::Get || Method == EMethod::Head)
	{
		curl_easy_setopt(Curl, CURLOPT_HTTPGET, 1L);
		if (Method == EMethod::Head)
		{
			curl_easy_setopt(Curl, CURLOPT_NOBODY, 1L);
		}
	}
	else if (Method == EMethod::Put)
	{
		curl_easy_setopt(Curl, CURLOPT_UPLOAD, 1L);
		curl_easy_setopt(Curl, CURLOPT_INFILESIZE, ContentLength);
		curl_easy_setopt(Curl, CURLOPT_READDATA, this);
		curl_easy_setopt(Curl, CURLOPT_READFUNCTION, &ReadBodyCallback);
		curl_easy_setopt(Curl, CURLOPT_SEEKDATA, this);
		curl_easy_setopt(Curl, CURLOPT_SEEKFUNCTION, &SeekBodyCallback);
	}
	else
	{
		check(Method == EMethod::Delete);
		curl_easy_setopt(Curl, CURLOPT_CUSTOMREQUEST, "DELETE");
	}

	curl_easy_setopt(Curl, CURLOPT_FOLLOWLOCATION, 1L);
	curl_easy_setopt(Curl, CURLOPT_NOSIGNAL, 1L);
	curl_easy_setopt(Curl, CURLOPT_URL, Url);

	// Headers
	curl_easy_setopt(Curl, CURLOPT_HTTPHEADER, CurlHeaders);
	curl_easy_setopt(Curl, CURLOPT_HEADERDATA, CurlHeaders);

	// Response
	curl_easy_setopt(Curl, CURLOPT_HEADERDATA, this);
	curl_easy_setopt(Curl, CURLOPT_HEADERFUNCTION, &WriteHeadersCallback);
	curl_easy_setopt(Curl, CURLOPT_WRITEDATA, this);
	curl_easy_setopt(Curl, CURLOPT_WRITEFUNCTION, &WriteBodyCallback);

	// Errors and logging
	ANSICHAR ErrorBuffer[CURL_ERROR_SIZE] = {0};
	curl_easy_setopt(Curl, CURLOPT_ERRORBUFFER, ErrorBuffer);

	// SSL options
	curl_easy_setopt(Curl, CURLOPT_USE_SSL, CURLUSESSL_ALL);
	curl_easy_setopt(Curl, CURLOPT_SSL_VERIFYPEER, 1);
	curl_easy_setopt(Curl, CURLOPT_SSL_VERIFYHOST, 1);
	curl_easy_setopt(Curl, CURLOPT_SSLCERTTYPE, "PEM");

	// SSL certification verification
	curl_easy_setopt(Curl, CURLOPT_CAINFO, nullptr);
	curl_easy_setopt(Curl, CURLOPT_SSL_CTX_FUNCTION, *SslContextCallback);
	curl_easy_setopt(Curl, CURLOPT_SSL_CTX_DATA, this);

	// Send the request
	CURLcode CurlResult = curl_easy_perform(Curl);

	// Free the headers object
	curl_slist_free_all(CurlHeaders);
	curl_easy_setopt(Curl, CURLOPT_HEADERDATA, nullptr);

	// Get the response code
	long ResponseCode = 0;
	if (CurlResult == CURLE_OK)
	{
		CurlResult = curl_easy_getinfo(Curl, CURLINFO_RESPONSE_CODE, &ResponseCode);
	}
	
	if (CurlResult != CURLE_OK)
	{
		UE_LOGF(LogS3Client, Error, "%s", ErrorBuffer);
	}

	const char* ErrorMsg = curl_easy_strerror(CurlResult);

	if (const uint64 Size = ResponseBody.TotalSize(); Size > 0)
	{
		return FS3Response
		{
			static_cast<uint32>(ResponseCode),
			static_cast<uint32>(CurlResult),
			FSharedBuffer::TakeOwnership(ResponseBody.ReleaseOwnership(), Size, FMemory::Free)
		};
	}
	else
	{
		return FS3Response
		{
			static_cast<uint32>(ResponseCode),
			static_cast<uint32>(CurlResult)
		};
	}
}

int FS3Client::FS3Request::StatusCallback(void* Ptr, curl_off_t TotalDownloadSize, curl_off_t CurrentDownloadSize, curl_off_t TotalUploadSize, curl_off_t CurrentUploadSize)
{
	return 0;
}

size_t FS3Client::FS3Request::WriteHeadersCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
{
	const size_t WriteSize = SizeInBlocks * BlockSizeInBytes;
	if (WriteSize > 0)
	{
		FS3Client::FS3Request& Request = *static_cast<FS3Client::FS3Request*>(UserData);
		Request.ResponseHeader.Append((const ANSICHAR*)Ptr, WriteSize);
		return WriteSize;
	}
	return 0;
}

size_t FS3Client::FS3Request::ReadBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
{
	FS3Client::FS3Request& Request = *static_cast<FS3Client::FS3Request*>(UserData);
	check(Request.RequestBody.GetSize() > 0);

	const uint8* RequestBody = reinterpret_cast<const uint8*>(Request.RequestBody.GetData());
	const size_t RequestBodySize = static_cast<size_t>(Request.RequestBody.GetSize());

	const size_t Offset = Request.BytesSent;
	const size_t ReadSize = FMath::Min(RequestBodySize - Offset, SizeInBlocks * BlockSizeInBytes);
	check(RequestBodySize >= Offset + ReadSize);

	FMemory::Memcpy(Ptr, RequestBody + Offset, ReadSize);
	Request.BytesSent += ReadSize;

	return ReadSize;
}

size_t FS3Client::FS3Request::SeekBodyCallback(void* UserData, curl_off_t Offset, int Origin)
{
	auto& Request = *static_cast<FS3Client::FS3Request*>(UserData);

	uint64 Size = Request.RequestBody.GetSize();
	uint64 Position = Request.BytesSent;

	switch (Origin)
	{
	case SEEK_SET:	Position = Offset;				break;
	case SEEK_CUR:	Position = Position + Offset;	break;
	case SEEK_END:	Position = Size + Offset;		break;
	default:		return CURL_SEEKFUNC_FAIL;
	}

	if (Position > Size)
	{
		return CURL_SEEKFUNC_FAIL;
	}

	Request.BytesSent = Position;
	return CURL_SEEKFUNC_OK;
}

size_t FS3Client::FS3Request::WriteBodyCallback(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* UserData)
{
	const size_t WriteSize = SizeInBlocks * BlockSizeInBytes;
	if (WriteSize > 0)
	{
		FS3Client::FS3Request& Request = *static_cast<FS3Client::FS3Request*>(UserData);

		// If this is the first part of the body being received, try to reserve 
		// memory if content length is defined in the header.
		if (Request.ResponseBody.Tell() == 0)
		{
			static const ANSICHAR Prefix[] = "Content-Length: ";
			static const size_t PrefixLen = UE_ARRAY_COUNT(Prefix) - 1;

			for(const ANSICHAR* Header = *Request.ResponseHeader;;Header++)
			{
				// Check this header
				if (FCStringAnsi::Strnicmp(Header, Prefix, PrefixLen) == 0)
				{
					const size_t ContentLength = (size_t)atol(Header + PrefixLen);
					if (ContentLength > 0u)
					{
						Request.ResponseBody.Reserve(ContentLength);	
					}
					break;
				}

				// Move to the next string
				Header = FCStringAnsi::Strchr(Header, '\n');
				if (Header == nullptr)
				{
					break;
				}
			}
		}

		Request.ResponseBody.Serialize(Ptr, WriteSize);

		return WriteSize;
	}

	return 0;
}

int FS3Client::FS3Request::SslCertVerify(int PreverifyOk, X509_STORE_CTX* Context)
{
	if (PreverifyOk == 1)
	{
		SSL* Handle = static_cast<SSL*>(X509_STORE_CTX_get_ex_data(Context, SSL_get_ex_data_X509_STORE_CTX_idx()));
		check(Handle);

		SSL_CTX* SslContext = SSL_get_SSL_CTX(Handle);
		check(SslContext);

		FS3Client::FS3Request& Request = *static_cast<FS3Client::FS3Request*>(SSL_CTX_get_app_data(SslContext));
		if (!FSslModule::Get().GetCertificateManager().VerifySslCertificates(Context, *Request.Domain))
		{
			PreverifyOk = 0;
		}
	}

	return PreverifyOk;
}

CURLcode FS3Client::FS3Request::SslContextCallback(CURL* curl, void* sslctx, void* param)
{
	SSL_CTX* Context = static_cast<SSL_CTX*>(sslctx);
	const ISslCertificateManager& CertificateManager = FSslModule::Get().GetCertificateManager();

	CertificateManager.AddCertificatesToSslContext(Context);
	SSL_CTX_set_verify(Context, SSL_CTX_get_verify_mode(Context), SslCertVerify);
	SSL_CTX_set_app_data(Context, param);

	/* all set to go */
	return CURLE_OK;
}

const ANSICHAR* FS3Client::FS3Request::LexToString(EMethod Method)
{
	switch(Method)
	{
	case EMethod::Get:
		return "GET";
	case EMethod::Put:
		return "PUT";
	case EMethod::Delete:
		return "DELETE";
	case EMethod::Head:
		return "HEAD";
	default:
		check(false);
		return nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////
FS3Client::FS3Client(const FS3ClientConfig& ClientConfig, const FS3ClientCredentials& ClientCredentials)
	: Config(ClientConfig)
	, Credentials(ClientCredentials)
{
	ConnectionPool = MakeUnique<FS3Client::FConnectionPool>();

	if (!Config.Region.IsEmpty())
	{
		Config.ServiceUrl = TEXT("https://s3.amazonaws.com");
	}
}

FS3Client::~FS3Client()
{
}

void FS3Client::Setup(FS3Request& Request)
{
	Request.Curl = ConnectionPool->Alloc();
}

void FS3Client::Teardown(FS3Request& Request)
{
	ConnectionPool->Free(Request.Curl);
}

FS3GetObjectResponse FS3Client::GetObject(const FS3GetObjectRequest& GetRequest)
{
	TAnsiStringBuilder<256> Url;
	S3Client::AppendUrl(Url, Config.ServiceUrl, GetRequest.BucketName, GetRequest.Key);

	FS3Request Request(*this);
	return Request.Perform(FS3Request::EMethod::Get, Url.ToString(), FSharedBuffer());
}

FS3HeadObjectResponse FS3Client::HeadObject(const FS3HeadObjectRequest& GetRequest)
{
	TAnsiStringBuilder<256> Url;
	S3Client::AppendUrl(Url, Config.ServiceUrl, GetRequest.BucketName, GetRequest.Key);

	FS3Request Request(*this);
	return Request.Perform(FS3Request::EMethod::Head, Url.ToString(), FSharedBuffer());
}

FS3PutObjectResponse FS3Client::PutObject(const FS3PutObjectRequest& PutRequest)
{
	TAnsiStringBuilder<256> Url;
	S3Client::AppendUrl(Url, Config.ServiceUrl, PutRequest.BucketName, PutRequest.Key);

	FS3Request Request(*this);
	return Request.Perform(FS3Request::EMethod::Put, Url.ToString(), FSharedBuffer::MakeView(PutRequest.ObjectData));
}

FS3PutObjectResponse FS3Client::TryPutObject(const FS3PutObjectRequest& Request, int32 MaxAttempts, float Delay)
{
	FS3PutObjectResponse Response;
	for (int32 Attempt = 0; Attempt < MaxAttempts; ++Attempt)
	{
		Response = PutObject(Request);
		if (Response.IsOk())
		{
			break;
		}
		FPlatformProcess::Sleep(Delay);
	}

	return Response;
}

FS3ListObjectResponse FS3Client::ListObjects(const FS3ListObjectsRequest& ListRequest)
{
	TAnsiStringBuilder<256> Url;
	Url << StringCast<ANSICHAR>(*Config.ServiceUrl) << "/" << StringCast<ANSICHAR>(*ListRequest.BucketName); 

	char QueryParamDelim = '?';
	auto AppendQueryDelim = [&](FAnsiStringBuilderBase& Sb) -> FAnsiStringBuilderBase&
	{
		Sb << QueryParamDelim;
		QueryParamDelim = '&';
		return Sb;
	};

	if (ListRequest.Delimiter != 0)
	{
		AppendQueryDelim(Url) << "delimiter=" << char(ListRequest.Delimiter);
	}

	if (ListRequest.Marker.IsEmpty() == false)
	{
		AppendQueryDelim(Url) << "marker=" << ListRequest.Marker; 
	}

	if (ListRequest.MaxKeys > 0)
	{
		AppendQueryDelim(Url) << "max-keys=" << ListRequest.MaxKeys; 
	}

	if (!ListRequest.Prefix.IsEmpty())
	{
		AppendQueryDelim(Url) << "prefix=" << StringCast<ANSICHAR>(*ListRequest.Prefix);
	}

	FS3Request Request(*this);
	FS3Response Response = Request.Perform(FS3Request::EMethod::Get, Url.ToString(), FSharedBuffer());
	if (!Response.IsOk())
	{
		return FS3ListObjectResponse(MoveTemp(Response));
	}

	if (Response.GetBody().GetSize() == 0)
	{
		return FS3ListObjectResponse{MoveTemp(Response), FString(), TArray<FS3Object>()};
	}

	FString Body(Response.ToString());

	FXmlFile XmlFile(Body, EConstructMethod::ConstructFromBuffer);
	if (!XmlFile.IsValid())
	{
		//TODO: Better error message
		return FS3ListObjectResponse{{500, MoveTemp(Response)}};
	}

	const FXmlNode* Root = XmlFile.GetRootNode();
	if (!Root)
	{
		//TODO: Better error message
		return FS3ListObjectResponse{{500, MoveTemp(Response)}};
	}

	FString BucketName;
	FString NextMarker;
	bool bIsTruncated = false;
	TArray<FS3Object> Objects;

	const TArray<FXmlNode*>& Children = Root->GetChildrenNodes();
	for (const FXmlNode* Child : Children)
	{
		if (Child->GetTag() == TEXT("Name"))
		{
			BucketName = Child->GetContent();
		}
		else if (Child->GetTag() == TEXT("IsTruncated"))
		{
			bIsTruncated = Child->GetContent().Contains(TEXT("True"), ESearchCase::IgnoreCase);
		}
		else if (Child->GetTag() == TEXT("NextMarker"))
		{
			NextMarker = Child->GetContent();
		}
		else if (Child->GetTag() == TEXT("Contents"))
		{
			FS3Object& S3Object = Objects.AddDefaulted_GetRef();
			
			const TArray<FXmlNode*>& ObjectInfoNodes = Child->GetChildrenNodes();
			for (const FXmlNode* InfoNode : ObjectInfoNodes )
			{
				if (InfoNode->GetTag() == TEXT("Key"))
				{
					S3Object.Key = InfoNode->GetContent();
				}
				else if (InfoNode->GetTag() == TEXT("Size"))
				{
					S3Object.Size = FCString::Strtoui64(*InfoNode->GetContent(), nullptr, 10);
				}
				else if (InfoNode->GetTag() == TEXT("LastModified"))
				{
					S3Object.LastModifiedText = InfoNode->GetContent();
					FDateTime::ParseIso8601(*S3Object.LastModifiedText, S3Object.LastModified);
				}
			}
		}
	}

	return FS3ListObjectResponse{
		{200, 0, FSharedBuffer()},
		MoveTemp(BucketName),
		MoveTemp(Objects),
		MoveTemp(NextMarker),
		bIsTruncated};
}

FS3DeleteObjectResponse FS3Client::DeleteObject(const FS3DeleteObjectRequest& DeleteRequest)
{
	TAnsiStringBuilder<256> Url;
	S3Client::AppendUrl(Url, Config.ServiceUrl, DeleteRequest.BucketName, DeleteRequest.Key);

	FS3Request Request(*this);
	return Request.Perform(FS3Request::EMethod::Delete, Url.ToString(), FSharedBuffer());
}

} // namespace UE

#undef UE_ENABLE_S3_DEBUGGING

#endif // (IS_PROGRAM || WITH_EDITOR)
