// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class S3Client : ModuleRules
{
	public S3Client(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"Sockets",
			"SSL",
			"libcurl",
			"Json",
			"XmlParser"
		});

		AddEngineThirdPartyPrivateStaticDependencies(Target, "libcurl");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
